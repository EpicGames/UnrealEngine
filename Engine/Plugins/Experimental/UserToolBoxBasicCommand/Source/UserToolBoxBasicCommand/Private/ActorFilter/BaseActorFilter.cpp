// Copyright Epic Games, Inc. All Rights Reserved.


#include "ActorFilter/BaseActorFilter.h"
#include "DatasmithAssetUserData.h"
#include "Components/StaticMeshComponent.h"
#include "DatasmithContentBlueprintLibrary.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Editor.h"
#include "Engine/StaticMeshActor.h"

void GetAttachedActors( AActor* Actor, bool bRecurseChildren, TSet< AActor* >& OutActors )
{
	TArray< AActor* > ChildrenActors;
	Actor->GetAttachedActors( ChildrenActors );
	for ( AActor* ChildActor : ChildrenActors )
	{
		OutActors.Add( ChildActor );

		if ( bRecurseChildren )
		{
			GetAttachedActors( ChildActor, bRecurseChildren, OutActors );
		}
	}
}
TArray<AActor*> GetDescendantsAtN(AActor* Actor, int Rank, bool AddIntermediaries)
{
	const int CurrentRank=Rank-1;
	TArray<AActor*> Result;
	TArray<AActor*> Children;
	Actor->GetAttachedActors(Children);
	if (CurrentRank>0)
	{
		for (AActor* Child:Children)
		{
			Result.Append(GetDescendantsAtN(Child,CurrentRank,AddIntermediaries));
		}	
	}
	else
	{
		Result.Add(Actor);
	}
	if (AddIntermediaries)
	{
		Result.AddUnique(Actor);
	}
	return Result;
	
	
	
	
}

TArray<AActor*> UBaseActorFilter::Filter_Implementation(const TArray<AActor*>& Source)
{
	return Source;
}

bool UBaseActorFilter::FilterUnit_Implementation(AActor* Source)
{
	return true;
}

 TArray<AActor*> UGetAllDescendants::FilterImpl(const TArray<AActor*>& Source)
{
	TSet<AActor*> Result;
	for (AActor* Actor:Source)
	{
		GetAttachedActors(Actor,true,Result);
	}
	return Result.Array();
}

 TArray<AActor*> UGetParents::FilterImpl(const TArray<AActor*>& Source)
{
	TSet<AActor*> Result;
	for (AActor* Actor:Source)
	{
		Result.Add(Actor->GetAttachParentActor());
	}
	Result.Remove(nullptr);
	return Result.Array();
}

 TArray<AActor*> UHasAttachedActor::FilterImpl(const TArray<AActor*>& Source)
{
	TSet<AActor*> Result;
	for (AActor* Actor:Source)
	{
		TArray<AActor*> Children;
		Actor->GetAttachedActors(Children);
		if (Children.Num()>0)
		{
			Result.Add(Actor);
		}
			
	}
	return Result.Array();
}

 TArray<AActor*> UHasComponentOfClass::FilterImpl(const TArray<AActor*>& Source)
{
	TSet<AActor*> Result;
	for (AActor* Actor:Source)
	{
		TArray<UActorComponent*> Components;
		Actor->GetComponents(ComponentClass,Components);
		if (Components.Num()>0)
		{
			Result.Add(Actor);
		}
			
	}
	return Result.Array();
}

 TArray<AActor*> UHasMetadataByKey::FilterImpl(const TArray<AActor*>& Source)
{
	TSet<AActor*> Result;
	for (AActor* Actor:Source)
	{
		FString Value=UDatasmithContentBlueprintLibrary::GetDatasmithUserDataValueForKey(Actor,Key);
		if (!Value.IsEmpty())
		{
			Result.Add(Actor);
		}
			
	}
	return Result.Array();
}

 TArray<AActor*> UHasMetadataByKeyAndValue::FilterImpl(const TArray<AActor*>& Source)
{
	TSet<AActor*> Result;
	for (AActor* Actor:Source)
	{
		FString RetValue=UDatasmithContentBlueprintLibrary::GetDatasmithUserDataValueForKey(Actor,Key);
		if (RetValue==Value)
		{
			Result.Add(Actor);
		}
			
	}
	return Result.Array();
}

 TArray<AActor*> UIsClassOf::FilterImpl(const TArray<AActor*>& Source)
{
	TSet<AActor*> Result;
	for (AActor* Actor:Source)
	{
		if ( Actor->GetClass()==ActorClass || (ChildClass && Actor->GetClass()->IsChildOf(ActorClass)))
		{
			Result.Add(Actor);
		}
			
	}
	return Result.Array();
}

 TArray<AActor*> UGetNDescendants::FilterImpl(const TArray<AActor*>& Source)
{
	TSet<AActor*> Result;
	for (AActor* Actor:Source)
	{
		Result.Append(GetDescendantsAtN(Actor,N+1,AddIntermediaries));
	}
	return Result.Array();
}

TArray<AActor*> UHasMetadataByKeyAndValueDropDown::FilterImpl(const TArray<AActor*>& Source)
{
	TSet<AActor*> Result;
	TMap<FString,TArray<AActor*>> ActorSortByValue;
	TArray<UDatasmithAssetUserData*> UserDatas;
	TSet<FName> TotalKeys;
    UDatasmithContentBlueprintLibrary::GetAllDatasmithUserData(AActor::StaticClass(),UserDatas);
	for (UDatasmithAssetUserData* UserData:UserDatas)
	{
		TArray<FName> Keys;
		UserData->MetaData.GenerateKeyArray(Keys);
		TotalKeys.Append(Keys);
	}
	FName SelectedKey=NAME_None;
	TSharedRef<SVerticalBox> KeyList = SNew(SVerticalBox);
	const ISlateStyle& EditorStyle=FAppStyle::Get();
	TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title(FText::FromString("Select a Key"))
		.ClientSize(FVector2D(100.f, FMath::Max(25.f * ActorSortByValue.Num(),300.0f)))
		.IsTopmostWindow(true);

	for (FName Key : TotalKeys) {
		KeyList->AddSlot()
			[
				SNew(SButton)
				.Text(FText::FromName(Key))
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.ButtonStyle(EditorStyle, "PropertyEditor.AssetComboStyle")
				.ForegroundColor(EditorStyle.GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
				.OnClicked_Lambda([Key, Window, &SelectedKey]()->FReply
				{
					SelectedKey=Key;
					Window->RequestDestroyWindow();
					return FReply::Handled();
				})
			];
	}
	Window->SetContent(
		SNew(SScrollBox)
		+SScrollBox::Slot()
		[
		KeyList
		]
		);
	GEditor->EditorAddModalWindow(Window.ToSharedRef());
	if (SelectedKey==NAME_None)
	{
		return TArray<AActor*>();	
	}
	
	

	
	for (AActor* Actor:Source)
	{
		FString Value=UDatasmithContentBlueprintLibrary::GetDatasmithUserDataValueForKey(Actor,SelectedKey);
		if (!Value.IsEmpty())
		{
			ActorSortByValue.FindOrAdd(Value).Add(Actor);
		}
	}
	
	 Window = SNew(SWindow)
		.Title(FText::FromString("Select a value"))
		.ClientSize(FVector2D(100.f, FMath::Max(25.f * ActorSortByValue.Num(),300.0f)))
		.IsTopmostWindow(true);

	TSharedRef<SVerticalBox> ValueList = SNew(SVerticalBox);
	TArray<FString> Values;
	ActorSortByValue.GenerateKeyArray(Values);
	FString SelectedValue;
	for (const FString& Value : Values) {
		ValueList->AddSlot()
			[
				SNew(SButton)
				.Text(FText::FromString(Value))
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.ButtonStyle(EditorStyle, "PropertyEditor.AssetComboStyle")
				.ForegroundColor(EditorStyle.GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
				.OnClicked_Lambda([Value, Window, &SelectedValue]()->FReply
				{
					SelectedValue=Value;
					Window->RequestDestroyWindow();
					return FReply::Handled();
				})
			];
	}
	Window->SetContent(
		SNew(SScrollBox)
		+SScrollBox::Slot()
		[
			ValueList
		]
		);
	GEditor->EditorAddModalWindow(Window.ToSharedRef());
	return ActorSortByValue.FindOrAdd(SelectedValue);

}
TArray<AActor*> UEmptyStaticMeshActor::FilterImpl(const TArray<AActor*>& Source)
{
	TArray<AActor*> Dest;
	for (AActor* Cur:Source)
	{
		if (Cur->IsA(AStaticMeshActor::StaticClass()) && Cast<AStaticMeshActor>(Cur)->GetStaticMeshComponent()->GetStaticMesh()==nullptr)
		{
			Dest.Add(Cur);
		}
	}
	return Dest;
}