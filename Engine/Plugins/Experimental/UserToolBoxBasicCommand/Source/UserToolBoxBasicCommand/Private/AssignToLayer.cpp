// Copyright Epic Games, Inc. All Rights Reserved.


#include "AssignToLayer.h"

#include "Layers/LayersSubsystem.h"
#include "Layers/Layer.h"
#include "Editor.h"
#include "UserToolBoxBasicCommand.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"

void UAssignToLayer::Execute()
{
	
	ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();

	if (LayersSubsystem == nullptr)
	{
		UE_LOGF(LogUserToolBoxBasicCommand, Warning, "Could no retrieve LayersSubsystem");
		return;
	}
	const ISlateStyle& EditorStyle=FAppStyle::Get();
	TArray<TWeakObjectPtr<ULayer>> Layers;
	LayersSubsystem->AddAllLayersTo(Layers);

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(NSLOCTEXT("UTBAssignToLayer", "SelectLayer", "Select Layer"))
		.ClientSize(FVector2D(100.f, 30.f * (Layers.Num()+1)))
		.IsTopmostWindow(true);

	TSharedRef<SVerticalBox> LayerList = SNew(SVerticalBox)
	+SVerticalBox::Slot()
	[
			SNew(SEditableTextBox)
			.OnTextCommitted(FOnTextCommitted::CreateLambda([Window, LayersSubsystem](const FText& NewText,ETextCommit::Type CommitType)
			{
				LayersSubsystem->AddSelectedActorsToLayer(*NewText.ToString());
					Window->RequestDestroyWindow();
			}))
	];

	
	for (TWeakObjectPtr<ULayer> Layer : Layers) {
		LayerList->AddSlot()
			[
				SNew(SButton)
				.Text(FText::FromName(GetLayerName(Layer)))
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.ButtonStyle(EditorStyle, "PropertyEditor.AssetComboStyle")
				.ForegroundColor(EditorStyle.GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
				.OnClicked_Lambda([Layer, Window, LayersSubsystem, this]()->FReply
				{
					LayersSubsystem->AddSelectedActorsToLayer(GetLayerName(Layer));

					Window->RequestDestroyWindow();
					return FReply::Handled();
				})
			];
	}

	Window->SetContent(LayerList);

	GEditor->EditorAddModalWindow(Window);

	return;
}

FName UAssignToLayer::GetLayerName(TWeakObjectPtr<ULayer> Layer)
{
	return Layer->GetLayerName();

}

UAssignToLayer::UAssignToLayer()
{
	Name="Assign to layer";
	Tooltip="Assign selected actor to a specific layer";
	Category="Level";
}
