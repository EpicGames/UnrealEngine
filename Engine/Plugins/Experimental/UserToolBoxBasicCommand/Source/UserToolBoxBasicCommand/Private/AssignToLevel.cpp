// Copyright Epic Games, Inc. All Rights Reserved.


#include "AssignToLevel.h"
#include "Engine/Selection.h"
#include "EditorLevelUtils.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Widgets/Input/SButton.h"
void UAssignToLevel::Execute()
{
	const ISlateStyle& EditorStyle=FAppStyle::Get();
	
	TArray<ULevel*> Levels=GEditor->GetEditorWorldContext().World()->GetLevels();

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(NSLOCTEXT("UTBAssignToLevel", "SelectLevel", "Select Level"))
		.ClientSize(FVector2D(100.f, 25.f * Levels.Num()))
		.IsTopmostWindow(true);

	TSharedRef<SVerticalBox> LevelList = SNew(SVerticalBox);

	for (ULevel* Level : Levels) {
		FString FullLevelName = Level->GetFullName();
		FString LeftS, RightS;
		FString SearchString(TEXT("."));

		FullLevelName.Split(SearchString, &LeftS, &RightS);
		SearchString = TEXT(":");
		RightS.Split(SearchString, &LeftS, &RightS);

		LevelList->AddSlot()
			[
				SNew(SButton)
				.Text(FText::FromString(LeftS))
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.ButtonStyle(EditorStyle, "PropertyEditor.AssetComboStyle")
				.ForegroundColor(EditorStyle.GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
				.OnClicked_Lambda([Level,Window]()->FReply
				{
					EditorLevelUtils::MoveSelectedActorsToLevel(Level);
					Window->RequestDestroyWindow();
					return FReply::Handled();
				})
			];
	}
	Window->SetContent(LevelList);
	GEditor->EditorAddModalWindow(Window);
	return;
}

UAssignToLevel::UAssignToLevel()
{
	Name="Assign to level";
	Tooltip="Assign selected actor into a specific sublevel";
	Category="Level";
}
