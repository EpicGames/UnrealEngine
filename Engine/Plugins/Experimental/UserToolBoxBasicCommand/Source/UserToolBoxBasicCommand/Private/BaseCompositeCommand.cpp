// Copyright Epic Games, Inc. All Rights Reserved.


#include "BaseCompositeCommand.h"
#include "EditorUtilityBlueprint.h"
#include "UserToolBoxBaseBlueprint.h"
#include "UTBBaseTab.h"
#include "Engine/Selection.h"
#include "UserToolBoxBasicCommand.h"

UUTBBaseCommand* UBaseCompositeCommand::CopyCommand(UObject* Owner) const
{
	UUTBTabSection* Section=Cast<UUTBTabSection>(Owner);
	if (!IsValid(Section))
	{
		UE_LOGF(LogUserToolBoxBasicCommand, Error, "The new owner %ls isn't a tab section", *Owner->GetName());
		return nullptr;
	}
	UUserToolBoxBaseTab* NewTab= Cast<UUserToolBoxBaseTab>(Section->GetOuter());
	if (!IsValid(NewTab))
	{
		UE_LOGF(LogUserToolBoxBasicCommand, Error, "The owner %ls of the section isn't a tab ", Section->GetOuter()?(*Section->GetOuter()->GetName()):TEXT("Null"));
		return nullptr;
	}
	UBaseCompositeCommand* NewToggleCommand=DuplicateObject(this,Owner);
	if (NewTab==this->GetOuter()->GetOuter())
	{
		//
	}
	else
	{
		NewToggleCommand->Commands.Empty(Commands.Num());
		UUTBTabSection* PlaceHolderSection=NewTab->GetPlaceHolderSection();
		for (UUTBBaseCommand* CurrentCommand:Commands)
		{
			UUTBBaseCommand* NewSubCommand=CurrentCommand->CopyCommand(PlaceHolderSection);
			NewToggleCommand->Commands.Add(NewSubCommand);
			PlaceHolderSection->Commands.Add(NewSubCommand);
		}	
	}
	Section->Commands.Add(NewToggleCommand);
	return NewToggleCommand;
}