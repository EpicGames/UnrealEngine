// Copyright Epic Games, Inc. All Rights Reserved.


#include "ChangeViewMode.h"

#include "LevelEditorViewport.h"

void UChangeViewMode::Execute()
{
	const TArray<FLevelEditorViewportClient*>& VCs=GEditor->GetLevelViewportClients();
	for (FLevelEditorViewportClient* VC:VCs)
	{
		VC->SetViewMode(ViewMode);
	}
		
	
	return Super::Execute();
}

UChangeViewMode::UChangeViewMode()
{
	Name="Change Viewport mode";
	Category="Viewport";
	Tooltip="Change the main viewport mode ( lit, unlit ...)";
	
}
