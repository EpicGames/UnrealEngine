// Copyright Epic Games, Inc. All Rights Reserved.


 #include "CleanHierarchy.h"
#include "Components/MeshComponent.h"
#include "DatasmithContentBlueprintLibrary.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"

int GetAttachedChildCount(AActor* Actor)
{
	TArray< AActor* > ChildrenActors;
	Actor->GetAttachedActors( ChildrenActors );
	return ChildrenActors.Num();
}
void GetAttachedActorsRecursive( AActor* Actor, bool bRecurseChildren, TSet< AActor* >& OutActors )
{
	TArray< AActor* > ChildrenActors;
	Actor->GetAttachedActors( ChildrenActors );
	for ( AActor* ChildActor : ChildrenActors )
	{
		OutActors.Add( ChildActor );
		if ( bRecurseChildren )
		{
			GetAttachedActorsRecursive( ChildActor, bRecurseChildren, OutActors );
		}
	}
}
UCleanHierarchy::UCleanHierarchy()
{
	Name="Clean Hierarchy";
	Tooltip="Remove geometry-less actor from the selected node";
	Category="Scene";
}

void UCleanHierarchy::Execute()
{
	UWorld* World = GEngine->GetWorldContexts()[0].World();

	USelection* SelectedActors = GEditor->GetSelectedActors();
	TArray<AActor*> Actors;
	SelectedActors->GetSelectedObjects(Actors);
	TSet<AActor*> GeometryLessActors;
	TSet<AActor*> ChildLessActors;
	TSet<AActor*> ActorsToProcess;
	for (AActor* CurrentActor:Actors)
	{
		GetAttachedActorsRecursive(CurrentActor,true,ActorsToProcess);
	}
	for (AActor* CurrentActor:ActorsToProcess)
	{
		
		TArray<UActorComponent*> Components;
		CurrentActor->GetComponents(UMeshComponent::StaticClass(),Components,false);
		if (Components.IsEmpty())
		{
			bool FoundMetadata=false;
			for (FString Key:DSMetadataToPreserve)
			{
				if (!UDatasmithContentBlueprintLibrary::GetDatasmithUserDataValueForKey(CurrentActor,FName(Key)).IsEmpty())
				{
					FoundMetadata=true;
					break;
				}
			}
			if (!FoundMetadata)
			{
				GeometryLessActors.Add(CurrentActor);
			}
		}	
		if (GetAttachedChildCount(CurrentActor)==0 && GeometryLessActors.Contains(CurrentActor))
		{
			ChildLessActors.Add(CurrentActor);
		}	
		
	}
	TSet<AActor*> ToRemove;

	//first we remove empty branches
	TArray<AActor*> WorkingArray(ChildLessActors.Array());
	while (!WorkingArray.IsEmpty())
	{
		TArray<AActor*> Parents;
		for (AActor* CurrentActor:WorkingArray)
		{
			if (GetAttachedChildCount(CurrentActor)!=0)
			{
				continue;
			}
			AActor* Parent=CurrentActor->GetAttachParentActor();
			if (Parent && GeometryLessActors.Contains(Parent))
			{
				Parents.Add(Parent);
			}
			CurrentActor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
			ToRemove.Add(CurrentActor);
			GeometryLessActors.Remove(CurrentActor);
		}
		WorkingArray=Parents;
	}
	for (AActor* ElementToRemove:ToRemove)
	{
		ElementToRemove->Destroy();
	}
	// now we are looking for intermediary
	for (AActor* CurGeometryLess:GeometryLessActors)
	{
		int ChildCount=GetAttachedChildCount(CurGeometryLess);
		if (ChildCount!=0)
		{
			//finding the best parent
			AActor* PotentialNewParent=CurGeometryLess->GetAttachParentActor();
			bool FoundParent=false;
			while (PotentialNewParent->IsValidLowLevel())
			{
				if (GeometryLessActors.Contains(PotentialNewParent))
				{
					PotentialNewParent=PotentialNewParent->GetAttachParentActor();
					continue;
				}
				break;
			}
			TArray<AActor*> Children;
			CurGeometryLess->GetAttachedActors(Children);
			for (AActor* Child:Children)
			{
				if (PotentialNewParent==nullptr)
				{
					Child->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
				}
				else
				{
					Child->AttachToActor(PotentialNewParent,FAttachmentTransformRules::KeepWorldTransform);
				}
			}
		}
	}
	for (AActor* ElementToRemove:GeometryLessActors)
	{
		ElementToRemove->Destroy();
	}
}
