// Copyright Epic Games, Inc. All Rights Reserved.


#include "CompositeCommand.h"


#include "EditorUtilityBlueprint.h"
#include "EngineUtils.h"
#include "UserToolBoxBaseBlueprint.h"
#include "UTBBaseTab.h"
#include "Engine/Selection.h"
#include "UserToolBoxBasicCommand.h"

void UCompositeCommand::Execute()
{

	for (UUTBBaseCommand* Command:Commands)
	{
		
		
		if (!IsValid(Command))
		{
			continue;	
		}
		Command->ExecuteCommand();
	}
}

