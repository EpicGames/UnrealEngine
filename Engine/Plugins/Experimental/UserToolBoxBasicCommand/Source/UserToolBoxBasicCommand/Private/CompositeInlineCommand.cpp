// Copyright Epic Games, Inc. All Rights Reserved.


#include "CompositeInlineCommand.h"




void UCompositeInlineCommand::Execute()
{

	for (UUTBBaseCommand* Command:Commands)
	{
		
		
		if (!IsValid(Command))
		{
			continue;	
		}
		Command->ExecuteCommand();
	}
}

