// Copyright Epic Games, Inc. All Rights Reserved.


#include "ConsoleVariable.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "Kismet/KismetSystemLibrary.h"
#include "UserToolBoxBasicCommand.h"
void UConsoleVariable::Execute()
{
	
	for (const FString& Command : ConsoleCommands)
	{
		UE_LOGF(LogUserToolBoxBasicCommand, Log, "Executing Console Command \"%ls\" .", *Command);
		UKismetSystemLibrary::ExecuteConsoleCommand(GetWorld(), Command, nullptr);
	}
	
}
