// Copyright Epic Games, Inc. All Rights Reserved.


#include "BaseCompositeCommandCustomization.h"
#include "BaseCompositeCommand.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor/UTBTabEditor.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "SDropTarget.h"
#include "Widgets/Text/STextBlock.h"

void FBaseCompositeCommandCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& CompositeCommandCategory = DetailBuilder.EditCategory("Composite Commands");

	TSharedRef<IPropertyHandle> CommandsProperty = DetailBuilder.GetProperty("Commands");

	TSharedRef<FDetailArrayBuilder> CommandArrayBuilder = MakeShareable( new FDetailArrayBuilder(CommandsProperty ) );
	CommandArrayBuilder->OnGenerateArrayElementWidget( FOnGenerateArrayElementWidget::CreateSP(this, &FBaseCompositeCommandCustomization::GenerateArrayChildren) );

	

	
	CompositeCommandCategory.AddCustomBuilder( CommandArrayBuilder, false );
	
}

void FBaseCompositeCommandCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CustomizeDetails(*DetailBuilder);
}

void FBaseCompositeCommandCustomization::GenerateArrayChildren(TSharedRef<IPropertyHandle> PropertyHandle,
	int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
{
	UObject* CommandAsUObject=nullptr;
	PropertyHandle->GetValue(CommandAsUObject);
	UUTBBaseCommand* Command=Cast<UUTBBaseCommand>(CommandAsUObject);
	TSharedPtr<STextBlock> TextBlock=SNew(STextBlock)
			.Text(TAttribute<FText>::Create([Command]()
			{
				if (IsValid(Command))
				{
					return FText::FromString(Command->Name);
				}
				return FText::FromString("Drag command");
			}));
	ChildrenBuilder.AddCustomRow(FText::FromString("Command"))
	.ValueWidget
	[
		
		SNew(SDropTarget)
		.OnAllowDrop(SDropTarget::FVerifyDrag::CreateLambda([](TSharedPtr<FDragDropOperation> op)
		{
			if (op->IsOfType<FCommandDragDropOperation>())
			{
				TSharedPtr<FCommandDragDropOperation> CommandOp=StaticCastSharedPtr<FCommandDragDropOperation>(op);
				if (!CommandOp.IsValid())
				{
					return false;
				}
					
				if (CommandOp->Command->GetClass()->GetDefaultObject()==CommandOp->Command)
				{
					return false;
				}
				return true;
			}
			return false;
			
		}))
		.OnDropped(
			FOnDrop::CreateLambda([PropertyHandle, TextBlock](const FGeometry&,const FDragDropEvent& Event)
			{
				TSharedPtr<FCommandDragDropOperation> Op=Event.GetOperationAs<FCommandDragDropOperation>();

				
				
				if (Op.IsValid())
				{
					PropertyHandle->SetValue(Op->Command.Get());
					PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
					
					if (IsValid(Op->Command.Get()))
					{
						TextBlock->SetText(FText::FromString(Op->Command->Name));
					}else
					{
						TextBlock->SetText(FText::FromString("Drag command"));
					}
					
					return FReply::Handled().EndDragDrop();
				}
				return FReply::Unhandled();
		}))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				TextBlock.ToSharedRef()
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				SNew(SSpacer)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				PropertyHandle->CreateDefaultPropertyButtonWidgets()
				
			]

			
			
		]
	]
	.NameWidget
	[
	PropertyHandle->CreatePropertyNameWidget()
	]
	;
}
