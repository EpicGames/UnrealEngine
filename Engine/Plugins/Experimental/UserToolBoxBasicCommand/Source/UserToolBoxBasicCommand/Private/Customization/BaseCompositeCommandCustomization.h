// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"

/**
 * 
 */




class FBaseCompositeCommandCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FBaseCompositeCommandCustomization());
	}
	

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	void GenerateArrayChildren(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder);
};



