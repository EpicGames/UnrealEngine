// Copyright Epic Games, Inc. All Rights Reserved.


#include "BindableActionCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ExecuteBindableAction.h"
#include "IDetailChildrenBuilder.h"
#include "IHeadMountedDisplay.h"
#include "UTBBaseCommand.h"
#include "UTBBaseTab.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"






FBindableActionPropertyCustomization::~FBindableActionPropertyCustomization()
{
}

void FBindableActionPropertyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle,
	FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	BindingContexts.Empty();
	StructPropertyHandle=PropertyHandle;
	FInputBindingManager::Get().GetKnownInputContexts(BindingContexts);
	void* ValuePtr;
	StructPropertyHandle->GetValueData(ValuePtr);
	FBindableActionInfo* PropertyValue = reinterpret_cast<FBindableActionInfo*>(ValuePtr);
	TSharedPtr<IPropertyHandle> ParentPropertyHandle=StructPropertyHandle->GetParentHandle();
	if (PropertyValue)
	{
		if (PropertyValue->Context!="") 
		{
			for (TSharedPtr<FBindingContext> Context:BindingContexts)
			{
				if (Context->GetContextName()==FName(PropertyValue->Context))
				{
					SelectedContext=Context;
					FInputBindingManager::Get().GetCommandInfosFromContext(SelectedContext->GetContextName(),CurrentCommandInfos);
					CurrentCommand=TSharedPtr<FUICommandInfo>();
					for (TSharedPtr<FUICommandInfo> CommandInfo:CurrentCommandInfos)
					{
						if (CommandInfo->GetCommandName()==FName(PropertyValue->CommandName))
						{
							CurrentCommand=CommandInfo;
							break;
						}
					}	
				}
			}
		}
	}
	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			SNew(SComboBox<TSharedPtr<FBindingContext> >) // pose list combo
			.OptionsSource(&BindingContexts)
			.OnGenerateWidget(this, &FBindableActionPropertyCustomization::OnGenerateContextComboWidget)
			.OnSelectionChanged(this, &FBindableActionPropertyCustomization::OnSelectContext)
			[
				SNew(STextBlock).Text_Lambda([this]()
				{
					if (SelectedContext.IsValid())
					{
						return FText::FromName(SelectedContext->GetContextName());
					}
					return FText::FromString("No Context");
				})
			]
		]
		+SVerticalBox::Slot()
		[
			SNew(SComboBox<TSharedPtr<FUICommandInfo> >) // pose list combo
			.OptionsSource(&CurrentCommandInfos)
			.OnGenerateWidget(this, &FBindableActionPropertyCustomization::OnGenerateCommandComboWidget)
			.OnSelectionChanged(this, &FBindableActionPropertyCustomization::OnSelectCommand)
			[
				SNew(STextBlock).Text_Lambda([this]()
				{
					if (CurrentCommand.IsValid())
					{
						return FText::FromName(CurrentCommand->GetCommandName());
					}
					return FText::FromString("No Command");
				})
			]
		]
		
	];
	
	UE_LOGF(LogTemp,Display,"Header: %ls",*PropertyHandle->GetProperty()->GetName());
	
}

void FBindableActionPropertyCustomization::RefreshCommandList()
{
	if (SelectedContext.IsValid())
	{
		FInputBindingManager::Get().GetCommandInfosFromContext(SelectedContext->GetContextName(),CurrentCommandInfos);
	}
	else
	{
		CurrentCommandInfos.Empty();
	}
}
TSharedRef<SWidget> FBindableActionPropertyCustomization::OnGenerateContextComboWidget(TSharedPtr<FBindingContext> Item) const
{
	return SNew(STextBlock).Text(FText::FromName(Item->GetContextName()));
}

void FBindableActionPropertyCustomization::OnSelectContext(TSharedPtr<FBindingContext> Item, ESelectInfo::Type SelectInfo)
{
	if (Item.IsValid())
	{
		SelectedContext = Item;
		RefreshCommandList();
		if (CurrentCommand.IsValid())
		{
			if (!CurrentCommandInfos.Contains(CurrentCommand))
			{
				CurrentCommand.Reset();
			}
		}
	}
}

void FBindableActionPropertyCustomization::OnSelectCommand(TSharedPtr<FUICommandInfo> Item, ESelectInfo::Type SelectInfo)
{
	if (Item.IsValid())
	{
		CurrentCommand=Item;
		if (StructPropertyHandle.IsValid())
		{
			FBindableActionInfo Info;
			Info.Context=SelectedContext->GetContextName().ToString();
			Info.CommandName=CurrentCommand->GetCommandName().ToString();
			FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty());
			FString TextValue;
			StructProperty->Struct->ExportText(TextValue, &Info, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);
			StructPropertyHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags);
		}
	}
}

TSharedRef<SWidget> FBindableActionPropertyCustomization::OnGenerateCommandComboWidget(
	TSharedPtr<FUICommandInfo> Item) const
{
	return SNew(STextBlock).Text(FText::FromName(Item->GetCommandName()));
}

void FBindableActionPropertyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle,
                                                      IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	
}
