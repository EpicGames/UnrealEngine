// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ExecuteBindableAction.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "Framework/Commands/UICommandInfo.h"
/**
 * 
 */




class FBindableActionPropertyCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FBindableActionPropertyCustomization());
	}
	virtual ~FBindableActionPropertyCustomization() override;
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder,
	                               IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	
	TSharedRef<SWidget> OnGenerateContextComboWidget(TSharedPtr<FBindingContext> Item) const;
	void OnSelectContext(TSharedPtr<FBindingContext> Item, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> OnGenerateCommandComboWidget(TSharedPtr<FUICommandInfo> Item) const;
	void OnSelectCommand(TSharedPtr<FUICommandInfo> Item, ESelectInfo::Type SelectInfo);
private:
	TSharedPtr<FBindingContext> SelectedContext;
	TSharedPtr<FUICommandInfo> CurrentCommand;

	TArray<TSharedPtr<FBindingContext>> BindingContexts;
	TArray<TSharedPtr<FUICommandInfo>> CurrentCommandInfos;

	void RefreshCommandList();
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	
};



