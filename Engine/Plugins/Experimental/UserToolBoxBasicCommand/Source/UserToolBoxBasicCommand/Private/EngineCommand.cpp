// Copyright Epic Games, Inc. All Rights Reserved.


#include "EngineCommand.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
  void UEngineCommand::Execute()
 {
 	GEngine->Exec(GWorld, *Command);
 }