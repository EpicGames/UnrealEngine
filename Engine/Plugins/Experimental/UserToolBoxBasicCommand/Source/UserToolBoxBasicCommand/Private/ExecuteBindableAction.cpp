// Copyright Epic Games, Inc. All Rights Reserved.


#include "ExecuteBindableAction.h"

#include "UserToolBoxBasicCommand.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Application/SlateApplication.h"
UExecuteBindableAction::UExecuteBindableAction()
{
	Name="Exec Bindable Action";
	Tooltip="Execute a input bindable action";
	Category="Utility";
}

void UExecuteBindableAction::Execute()
{
	if (CommandInfo. Context.IsEmpty())
	{
		TArray<TSharedPtr<FBindingContext>> Contexts;
		FInputBindingManager::Get().GetKnownInputContexts(Contexts);
		for (TSharedPtr<FBindingContext> CurContext:Contexts)
		{
			UE_LOGF(LogUserToolBoxBasicCommand,Display,"%ls",*CurContext->GetContextName().ToString());
		}
		return;
	}
	TSharedPtr<FUICommandInfo> Command=FInputBindingManager::Get().FindCommandInContext(FName(CommandInfo.Context),FName(CommandInfo.CommandName));
	if (!Command.IsValid())
	{
		UE_LOGF(LogUserToolBoxBasicCommand,Warning,"%ls is not available, try these ones:",*CommandInfo.CommandName);
		TArray<TSharedPtr<FUICommandInfo>> CommandInfos;
		FInputBindingManager::Get().GetCommandInfosFromContext(FName(CommandInfo.Context),CommandInfos);
		for (TSharedPtr<FUICommandInfo> CurrentCommandInfo:CommandInfos)
		{
			UE_LOGF(LogUserToolBoxBasicCommand,Display,"%ls",*CurrentCommandInfo->GetCommandName().ToString());
		}
	}
	else
	{
		FInputChord OldInputChord;
		FInputChord TmpChord(FKey(FName("Subtract")),true,false,false,false);
		
		
		FInputBindingManager::Get().GetUserDefinedChord(FName(CommandInfo.Context),FName(CommandInfo.CommandName),EMultipleKeyBindingIndex::Secondary,OldInputChord);
		Command->SetActiveChord(TmpChord,EMultipleKeyBindingIndex::Secondary);
		FInputBindingManager::Get().NotifyActiveChordChanged(*Command.Get(),EMultipleKeyBindingIndex::Secondary);
		FModifierKeysState Modifier(true,false,false,false,false,false,false,false,false);
		
		FKeyEvent KeyEvent(FKey(FName("Subtract")),Modifier,0,false,45,109);
		FSlateApplication::Get().ProcessKeyDownEvent(KeyEvent);
		Command->SetActiveChord(OldInputChord,EMultipleKeyBindingIndex::Secondary);
		FInputBindingManager::Get().NotifyActiveChordChanged(*Command.Get(),EMultipleKeyBindingIndex::Secondary);
		
	}
	
}
