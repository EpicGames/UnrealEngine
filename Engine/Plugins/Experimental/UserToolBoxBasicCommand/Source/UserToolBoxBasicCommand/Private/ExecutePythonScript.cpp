// Copyright Epic Games, Inc. All Rights Reserved.


#include "ExecutePythonScript.h"
#include "Engine/Engine.h"

UExecutePythonScript::UExecutePythonScript()
{
	Name="Exec Python";
	Tooltip="Execute a python script";
	Category="Utility";
}

void UExecutePythonScript::Execute()
{
	FString CommandArg;
	if (!ScriptPath.FilePath.IsEmpty())
	{
		CommandArg=FString::Printf(TEXT("py \"%s\""),*ScriptPath.FilePath);
		if (!Args.IsEmpty())
		{
			CommandArg+= " " + FString::Printf(TEXT("\"%s\""),*Args);
		}
	}
	else
	{
		if (!Args.IsEmpty())
		{
			CommandArg+= "py " + Args;
		}	
	}
	
	GEngine->Exec(NULL, *CommandArg);
}
