// Copyright Epic Games, Inc. All Rights Reserved.


#include "FillStaticMeshActor.h"
#include "Engine/Selection.h"

#include "Editor.h"
#include "UserToolBoxBasicCommand.h"
#include "Components/StaticMeshComponent.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/StaticMeshActor.h"


void UFillStaticMeshActor::Execute()
{
	UWorld* World = GEngine->GetWorldContexts()[0].World();

	USelection* SelectedActors = GEditor->GetSelectedActors();
	TArray<AActor*> Actors;
	SelectedActors->GetSelectedObjects(Actors);
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry= AssetRegistryModule.Get();
	FARFilter Filter;
	Filter.bRecursivePaths=true;
	if (!RootPaths.IsEmpty())
	{
		for (FString RootPath:RootPaths)
		{
			Filter.PackagePaths.Add(FName(RootPath));	
		}
			
	}
	else
	{
		Filter.PackagePaths.Add(FName("/Game"));	
	}
	
	TArray<FAssetData> AssetDatas;
	FTopLevelAssetPath ClassPath(FName("/Script/Engine"),FName("StaticMesh"));
	Filter.ClassPaths.Add(ClassPath);
	AssetRegistry.GetAssets(Filter,AssetDatas);
	TMap<FName,FAssetData> MapOfStaticMeshByAssetName;
	AssetRegistry.GetAssets(Filter,AssetDatas);
	for (FAssetData& AssetData:AssetDatas)
	{
		MapOfStaticMeshByAssetName.Add(AssetData.AssetName,AssetData);
	}
	for (AActor* Actor: Actors)
	{
		if (Actor->IsA(AStaticMeshActor::StaticClass()))
		{
			AStaticMeshActor* StaticMeshActor=Cast<AStaticMeshActor>(Actor);
			UStaticMesh* CurrentSM=StaticMeshActor->GetStaticMeshComponent()->GetStaticMesh();
			if (CurrentSM==nullptr || !bAffectOnlyEmptyStaticMeshActor)
			{
				FString ActorLabel=Actor->GetActorLabel();
				FAssetData* FoundedAssetData=MapOfStaticMeshByAssetName.Find(FName(ActorLabel));
				if (FoundedAssetData!=nullptr)
				{
					UStaticMesh* StaticMesh=Cast<UStaticMesh>((*FoundedAssetData).GetAsset());
					if (StaticMesh!=nullptr)
					{
						StaticMeshActor->GetStaticMeshComponent()->SetStaticMesh(StaticMesh);
					}
				}
			}
		}
		
	}
}


UFillStaticMeshActor::UFillStaticMeshActor()
{
	Name="Fill StaticMeshActor";
	Tooltip="Fill SMA with empty Static Mesh with Mesh in the same name";
	Category="Actor";
}
