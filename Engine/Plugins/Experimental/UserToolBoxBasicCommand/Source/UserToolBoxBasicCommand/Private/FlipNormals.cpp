// Copyright Epic Games, Inc. All Rights Reserved.


#include "FlipNormals.h"

#include "UserToolBoxBasicCommand.h"

#include "UDynamicMesh.h"
#include "GeometryScript/MeshAssetFunctions.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Engine/Selection.h"
#include "Engine/StaticMeshActor.h"
#include "GeometryScript/MeshNormalsFunctions.h"
#include "Editor.h"
#include "Components/StaticMeshComponent.h"

void UFlipNormals::Execute()
{

	UDynamicMesh* DynamicMesh=NewObject<UDynamicMesh>();
	USelection* SelectedActors = GEditor->GetSelectedActors();
	TArray<AActor*> Actors;
	SelectedActors->GetSelectedObjects(Actors);
	FGeometryScriptCopyMeshFromAssetOptions OptionsFrom;
	OptionsFrom.bIgnoreRemoveDegenerates=true;
	FGeometryScriptCopyMeshToAssetOptions OptionsTo;
	
	FGeometryScriptMeshReadLOD Lod;
	FGeometryScriptMeshWriteLOD TargetLod;
	EGeometryScriptOutcomePins Result;
	for (AActor* Actor:Actors)
	{
		TArray<UStaticMeshComponent*> SelectedStaticMeshComponents;
		Actor->GetComponents(SelectedStaticMeshComponents);
		for (UStaticMeshComponent* SMC:SelectedStaticMeshComponents)
		{
			UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromStaticMesh(SMC->GetStaticMesh(),DynamicMesh,OptionsFrom,Lod,Result);
			if (Result==EGeometryScriptOutcomePins::Success)
			{
				UGeometryScriptLibrary_MeshNormalsFunctions::FlipNormals(DynamicMesh);
				AddObjectToTransaction(SMC->GetStaticMesh());
				UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshToStaticMesh(DynamicMesh,SMC->GetStaticMesh(),OptionsTo,TargetLod,Result);
			}
		}
		
	}
}

UFlipNormals::UFlipNormals()
{
	Name="Flip normals";
	Tooltip="Flip normals on each mesh used by selected actor";
	Category="Mesh";
}
