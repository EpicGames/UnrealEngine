// Copyright Epic Games, Inc. All Rights Reserved.


 #include "IsolateSelection.h"

#include "ActorEditorUtils.h"
#include "Components/MeshComponent.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "Engine/StaticMeshActor.h"
#include "EngineUtils.h"

UIsolateSelection::UIsolateSelection()
{
	Name="Isolate";
	Tooltip="Hide all actors except the selected ones";
	Category="Scene";
	bIsTransaction=true;
}

void UIsolateSelection::Execute()
{
	UWorld* World = GEngine->GetWorldContexts()[0].World();

	USelection* SelectedActors = GEditor->GetSelectedActors();

	bool IsIsolated = true;
	UClass* ClassToHide=bShouldOnlyAffectStaticMeshActor?AStaticMeshActor::StaticClass():AActor::StaticClass();
	for (TActorIterator<AActor> ActorIterator(World, ClassToHide); ActorIterator; ++ActorIterator)
	{
		AActor* Actor = CastChecked<AActor>(*ActorIterator);

		if (Actor == nullptr
			|| FActorEditorUtils::IsABuilderBrush(Actor)
			|| Actor->IsTemporarilyHiddenInEditor()
			)
		{
			continue;
		}

		TArray<UMeshComponent*> Components;
		Actor->GetComponents(Components);

		if (Components.Num() <= 0)
		{
			continue;
		}

		bool IsSelected = false;
		for (FSelectionIterator SelectedActorIterator(*SelectedActors); SelectedActorIterator; ++SelectedActorIterator)
		{
			AActor* SelectedActor = CastChecked<AActor>(*SelectedActorIterator);
			if (Actor == SelectedActor)
			{
				IsSelected = true;
				break;
			}
		}
		if (IsSelected)
		{
			continue;	
		}
		IsIsolated = false;
	}

	if (IsIsolated)
	{
		
		for (TActorIterator<AActor> ActorIterator(World, ClassToHide); ActorIterator; ++ActorIterator)
		{
			AActor* Actor = CastChecked<AActor>(*ActorIterator);
			if (Actor == nullptr || FActorEditorUtils::IsABuilderBrush(Actor))
			{
				continue;
			}

			TArray<UMeshComponent*> Components;
			Actor->GetComponents(Components);

			if (Components.Num() <= 0)
			{
				continue;	
			}

			SaveToTransactionBuffer(Actor, false);
			Actor->SetIsTemporarilyHiddenInEditor(false);
		}
	}
	else
	{
		//hide function
		{

			// Iterate through all of the actors and unhide them
			for (TActorIterator<AActor> ActorIterator(World, ClassToHide); ActorIterator; ++ActorIterator)
			{
				AActor* Actor = *ActorIterator;
				if (!FActorEditorUtils::IsABuilderBrush(Actor))
				{
					TArray<UMeshComponent*> Components;
					Actor->GetComponents(Components);
					// Save the actor to the transaction buffer to support undo/redo, but do
					// not call Modify, as we do not want to dirty the actor's package and
					// we're only editing temporary, transient values
					SaveToTransactionBuffer(Actor, false);
					if (Components.Num() > 0)
					{
						Actor->SetIsTemporarilyHiddenInEditor(true);
					}
				}
			}
			for (FSelectionIterator It(*SelectedActors); It; ++It)
			{
				AActor* Actor = static_cast<AActor*>(*It);
				checkSlow(Actor->IsA(AActor::StaticClass()));

				// Don't consider already hidden actors or the builder brush
				if (!FActorEditorUtils::IsABuilderBrush(Actor))
				{
					Actor->Modify();
					Actor->SetIsTemporarilyHiddenInEditor(false);
				}
			}
		}
	}
	
}
