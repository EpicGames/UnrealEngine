// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/Selection.h"
#include "EditorAssetLibrary.h"
#include "EditorLevelLibrary.h"
#include "Engine/StaticMeshActor.h"
#include "Editor.h"

#include "StaticMeshEditorSubsystem.h"
#include <functional>
#include "UserToolBoxBasicCommand.h"
#include "MergeCommand.h"
#include "Framework/Docking/TabManager.h"

UMerge::UMerge()
{
	Name="Merge";
	Tooltip="Merge selected actor together";
	Category="Mesh";
}

void UMerge::Execute()
{
	// handle advanced mode
	if (Advanced)
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FName("MergeActors"));
		return;
	}

	GEditor->BeginTransaction(FText::FromString("DeleteSelectedActors"));

	TArray<AStaticMeshActor*> ActorsToMerge;
	USelection* SelectedActors = GEditor->GetSelectedActors();
	for (FSelectionIterator ActorIterator(*SelectedActors); ActorIterator; ++ActorIterator)
	{
		AStaticMeshActor* StaticMeshActor = CastChecked<AStaticMeshActor>(*ActorIterator);
		if (StaticMeshActor == nullptr)
		{
			continue;	
		}
		ActorsToMerge.Push(StaticMeshActor);
	}

	FString NewName = "SM_MERGED_" + ActorsToMerge[0]->GetActorLabel();

	if (UEditorAssetLibrary::DoesAssetExist("/Game/" + NewName))
	{
		NewName += "_";
		int i = 0;
		while (UEditorAssetLibrary::DoesAssetExist("/Game/" + NewName + FString::FromInt(i)))
		{
			i++;
		}
		NewName += FString::FromInt(i);
	}

	UStaticMeshEditorSubsystem* StaticMeshEditorSubsystem = GEditor->GetEditorSubsystem<UStaticMeshEditorSubsystem>();
	if (StaticMeshEditorSubsystem == nullptr)
	{
		UE_LOGF(LogUserToolBoxBasicCommand, Warning, "Could no retrieve StaticMeshEditorSubsystem");
		return;
	}
	FMergeStaticMeshActorsOptions MergeOptions;
	auto MergeStaticMeshActors = std::bind(
		&UStaticMeshEditorSubsystem::MergeStaticMeshActors,
		StaticMeshEditorSubsystem,
		std::placeholders::_1,
		std::placeholders::_2,
		std::placeholders::_3
	);


	MergeOptions.BasePackageName = "/Game/" + NewName;
	MergeOptions.NewActorLabel = NewName;
	AStaticMeshActor* NewActor;
	
	MergeStaticMeshActors(ActorsToMerge, MergeOptions, NewActor);

	

	return;
}
