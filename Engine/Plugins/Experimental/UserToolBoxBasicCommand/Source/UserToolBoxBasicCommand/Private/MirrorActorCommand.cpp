// Copyright Epic Games, Inc. All Rights Reserved.


#include "MirrorActorCommand.h"
#include "Engine/Selection.h"
#include "Editor.h"
UMirrorActorCommand::UMirrorActorCommand()
{
	Name="Mirror actor";
	Tooltip="Create a mirror actor depending of the axis selectionned";
	Category="Actor";
	bIsTransaction=true;
}

void UMirrorActorCommand::Execute()
{

	
	USelection* SelectedActors = GEditor->GetSelectedActors();
	TArray<UObject*> ObjectsSelected;
	SelectedActors->GetSelectedObjects(ObjectsSelected);
	FVector Mirror(XAxis?-1.0f:1.0f,YAxis?-1.0f:1.0f,ZAxis?-1.0f:1.0f);
	if (!(XAxis || YAxis || ZAxis))
	{
		Mirror=FVector(-1.0,-1.0,-1.0);
	}
	for (UObject* Selected : ObjectsSelected)
	{
		AActor* InputActor = Cast<AActor>(Selected);

		//newly spawned actor identical to the input actor
		UWorld* World = InputActor->GetWorld();
		FActorSpawnParameters params;
		params.Template = InputActor;
		UClass* ItemClass = InputActor->GetClass();
		AddObjectToTransaction(World);
		AActor* const SpawnedActor = World->SpawnActor<AActor>(ItemClass, params);
		SpawnedActor->SetActorRelativeScale3D(SpawnedActor->GetActorRelativeScale3D()*Mirror);
		SpawnedActor->SetActorLabel("Mirrored_"+InputActor->GetActorLabel());
	}
	return Super::Execute();
}
