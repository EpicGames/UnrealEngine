// Copyright Epic Games, Inc. All Rights Reserved.


#include "PushComponentMaterialIntoMesh.h"
#include "Editor.h"
#include "Selection.h"
#include "Components/StaticMeshComponent.h"
void UPushComponentMaterialIntoMesh::Execute()
{
	USelection* SelectedActors = GEditor->GetSelectedActors();
	TArray<UObject*> ActorsAsObject;
	SelectedActors->GetSelectedObjects(AActor::StaticClass(),ActorsAsObject);
	TArray<UStaticMeshComponent*> SMCs;
	for (UObject* Object:ActorsAsObject)
	{
		AActor* Actor=Cast<AActor>(Object);
		if (IsValid(Actor))
		{
			TArray<UStaticMeshComponent*> Components;
			Actor->GetComponents(Components);
			for (UStaticMeshComponent* Component:Components)
			{
				UStaticMesh* StaticMesh=Cast<UStaticMesh>(Component->GetStaticMesh());
				if (!IsValid(StaticMesh))
				{
					return;
				}
				TArray<UMaterialInterface*> MatFromSMC=Component->GetMaterials();
				TArray<UMaterialInterface*> MatFromSM;
				for ( const FStaticMaterial& Material:StaticMesh->GetStaticMaterials())
				{
					MatFromSM.Add(Material.MaterialInterface);
				}
				if (MatFromSM!=MatFromSMC)
				{
					AddObjectToTransaction(StaticMesh);
					for (int Index=0;Index<MatFromSMC.Num();Index++ )
					{
						StaticMesh->SetMaterial(Index,MatFromSMC[Index]);
					
					}
				}
			}
		}
	}
}



UPushComponentMaterialIntoMesh::UPushComponentMaterialIntoMesh()
{
	Name="Material from component to mesh";
	Tooltip="for the current selection, push every material on a static mesh component into the static mesh";
	Category="Actor";
	bIsTransaction=true;
}
