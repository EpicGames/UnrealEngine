// Copyright Epic Games, Inc. All Rights Reserved.


#include "SelectActorByFilter.h"
#include "EngineUtils.h"
#include "Engine/Selection.h"
#include "ISinglePropertyView.h"
#include "UserToolBoxSubsystem.h"
#include "Dialog/SCustomDialog.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Engine/Engine.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "UserToolBoxSelectActorByFilter"
void USelectActorByFilter::Execute()
{
	UWorld* World = GEngine->GetWorldContexts()[0].World();
	USelection* SelectedActors = GEditor->GetSelectedActors();


	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs Args;
	Args.bAllowSearch=false;
	Args.bShowOptions=false;
	Args.bHideSelectionTip=true;
	Args.bShowKeyablePropertiesOption=false;
	Args.bShowObjectLabel=false;
	Args.bUpdatesFromSelection=false;
	Args.NameAreaSettings=FDetailsViewArgs::HideNameArea;
	TSharedPtr<SVerticalBox> VerticalBox; 
	TSharedRef<SCustomDialog> OptionsWindow =
		SNew(SCustomDialog)
		.Title(FText::FromString(Name+" properties"))
	.Buttons({
				SCustomDialog::FButton(LOCTEXT("OK", "OK")),
				SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel"))
		})
	.Content()
	[
			SNew(SBox)
		.MinDesiredWidth(400)
		[
		SAssignNew(VerticalBox,SVerticalBox)

		]
		
	];

	int ShowedFilterCount=0;
	for (FActorFilterOptions& Filter:FilterStack)
	{
		if (Filter.Filter==nullptr)
		{
			continue;
		}
		if (!Filter.ShowProperties)
		{
			continue;
		}
		TSharedRef<IDetailsView> DetailsView=PropertyEditorModule.CreateDetailView(Args);
		DetailsView->SetObject(Filter.Filter);	
		VerticalBox->AddSlot()
		[
			DetailsView
		];
		ShowedFilterCount++;
	}
	if (ShowedFilterCount>0)
	{
		const int32 PressedButtonIdx = OptionsWindow->ShowModal();
		if (PressedButtonIdx != 0)
		{
			
			return;
		}	
	}
	TArray<AActor*> CurrentSelection;
	if (ApplyToCurrentSelection)
	{
		SelectedActors->GetSelectedObjects(CurrentSelection);
		
	}else
	{
		for (TActorIterator<AActor> ActorIterator(World, AActor::StaticClass()); ActorIterator; ++ActorIterator)
		{
			CurrentSelection.Add(*ActorIterator);
		}
	}
	TArray<AActor*> OriginalSelection=CurrentSelection;
	for (FActorFilterOptions& Filter:FilterStack)
	{
		if (Filter.Filter==nullptr)
		{
			continue;
		}
			
		TArray<AActor*>Result=Filter.Filter->FilterImpl(Filter.Source==EActorFilterSource::OriginalSelection?OriginalSelection:CurrentSelection);
		if (Filter.Rule==EActorFilterRule::Add)
		{
			CurrentSelection.Append(Result);
		}else if (Filter.Rule==EActorFilterRule::Intersect)
		{
			TSet<AActor*> ResultSet=TSet<AActor*>(Result);
			TArray<AActor*> Intersect;
			for(AActor* Item:CurrentSelection)
			{
				if (ResultSet.Contains(Item))
				{
					Intersect.AddUnique(Item);
				}
			}
			CurrentSelection=Intersect;
		}else if (Filter.Rule==EActorFilterRule::Substract)
		{
			TSet<AActor*> ResultSet=TSet<AActor*>(Result);
			TArray<AActor*> Substract;
			for(AActor* Item:CurrentSelection)
			{
				if (!ResultSet.Contains(Item))
				{
					Substract.AddUnique(Item);
				}
			}
			CurrentSelection=Substract;
		}else if (Filter.Rule==EActorFilterRule::Replace)
		{
			CurrentSelection=Result;
		}
	}
	SelectedActors->Modify();
	SelectedActors->BeginBatchSelectOperation();
	SelectedActors->DeselectAll();
	for (AActor* Actor:CurrentSelection)
	{
		if (Actor!=nullptr)
		{
			SelectedActors->Select(Actor,true);
		}
	}
	SelectedActors->EndBatchSelectOperation();
	GEditor->NoteSelectionChange(true);
	return Super::Execute();
}

USelectActorByFilter::USelectActorByFilter()
{
	Name="Select actor by filter";
	Tooltip="Select actor from a stack of filter";
	Category="Actor";
	bIsTransaction=true;
}
#undef LOCTEXT_NAMESPACE