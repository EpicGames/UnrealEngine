// Copyright Epic Games, Inc. All Rights Reserved.


#include "SelectActorBySize.h"


#include "EngineUtils.h"
#include "SkeletalDebugRendering.h"
#include "Engine/Selection.h"
#include "UserToolBoxBasicCommand.h"
#include "Editor.h"

USelectActorBySize::USelectActorBySize()
{
	Name="Select actor by size";
	Tooltip="Select the actor depending of their bounding box size";
	Category="Actor";
	bIsTransaction=true;
}

void USelectActorBySize::Execute()
{
	UWorld* World = GEngine->GetWorldContexts()[0].World();

	
	TArray<AActor*> NewSelection;
	auto FilterPerBoundingBox = [&NewSelection, this](AActor* CurActor)
	{
		if (CurActor->GetClass()->HasAnyClassFlags(CLASS_NotPlaceable) )
		{
			return;
		}
		FVector BoundingBox, Origin;
		CurActor->GetActorBounds(false, Origin, BoundingBox, true);
		BoundingBox *= 2;
		float Size=SizeThreshold;
		if (BoundingBox.X < SizeThreshold
			&& BoundingBox.Y < SizeThreshold
			&& BoundingBox.Z < SizeThreshold
			)
		{
			NewSelection.Add(CurActor);
		}
	};

	for (TActorIterator<AActor> It(World, AActor::StaticClass()); It; ++It)
	{
		if (*It != nullptr)
		{
			FilterPerBoundingBox(*It);
		}
	}
	USelection* SelectedActors = GEditor->GetSelectedActors();
	AddObjectToTransaction(SelectedActors);
	SelectedActors->BeginBatchSelectOperation();
	SelectedActors->DeselectAll();
	for (AActor* CurActor : NewSelection)
	{
		SelectedActors->Select(CurActor);
	}
	SelectedActors->EndBatchSelectOperation();
	GEditor->NoteSelectionChange();
	
}
