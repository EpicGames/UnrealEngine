// Copyright Epic Games, Inc. All Rights Reserved.


#include "SetHighPrecisionOnMesh.h"
#include "Engine/Selection.h"
#include "Engine/StaticMesh.h"
#include "Editor.h"
#include "Components/StaticMeshComponent.h"

USetHighPrecisionOnMesh::USetHighPrecisionOnMesh()
{
	Name="Set HighPrecision";
	Tooltip=" Use HighPrecision on mesh used by selected actors";
	Category="Mesh";
	bIsTransaction=true;
}

void USetHighPrecisionOnMesh::Execute()
{
	USelection* SelectedActors = GEditor->GetSelectedActors();
	TArray<AActor*> Actors;
	TArray<UStaticMesh*> StaticMeshes;
	SelectedActors->GetSelectedObjects(Actors);
	for (AActor* Actor:Actors)
	{
		TArray<UStaticMeshComponent*> StaticMeshComponents;
		Actor->GetComponents(StaticMeshComponents);
		for (UStaticMeshComponent* StaticMeshComponent:StaticMeshComponents)
		{
			UStaticMesh* StaticMesh=StaticMeshComponent->GetStaticMesh();
			if (IsValid(StaticMesh))
			{
				StaticMeshes.Add(StaticMesh);
			}	
		}
		
	}
	for (UStaticMesh* StaticMesh:StaticMeshes)
	{
		AddObjectToTransaction(StaticMesh);
		for (int32 LODIndex = 0; LODIndex < StaticMesh->GetNumLODs(); LODIndex++)
		{
			FMeshBuildSettings& LODBuildSettings = StaticMesh->GetSourceModel(LODIndex).BuildSettings;
			LODBuildSettings.bUseHighPrecisionTangentBasis=bHighPrecisionTangent;
			LODBuildSettings.bUseFullPrecisionUVs=bHighPrecisionUV;
			StaticMesh->PostEditChange();
			StaticMesh->MarkPackageDirty();
		}
	}
	return;
}
