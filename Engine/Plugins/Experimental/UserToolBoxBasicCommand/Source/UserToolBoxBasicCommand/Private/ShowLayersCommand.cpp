// Copyright Epic Games, Inc. All Rights Reserved.



#include "ShowLayersCommand.h"
#include "Layers/Layer.h"
#include "Layers/LayersSubsystem.h"
#include "UserToolBoxBasicCommand.h"
#include "Editor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
UShowLayersCommand::UShowLayersCommand()
{
	Name="Show layers";
	Tooltip="Show specific layers, if isolate, hide the other ones";
	Category="Scene";
}

bool UShowLayersCommand::DisplayParameters()
{
	ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();

	if (LayersSubsystem == nullptr)
	{
		UE_LOGF(LogUserToolBoxBasicCommand, Warning, "Could no retrieve LayersSubsystem");
		return false;
	}
	const ISlateStyle& EditorStyle=FAppStyle::Get();
	TArray<TWeakObjectPtr<ULayer>> AvailableLayers;
	LayersSubsystem->AddAllLayersTo(AvailableLayers);

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(NSLOCTEXT("UTBAssignToLayer", "SelectLayer", "Select Layer"))
		.ClientSize(FVector2D(100.f, 30.f * (AvailableLayers.Num()+3)))
		.IsTopmostWindow(true);
	bool SelectionValid=false;
	TSharedRef<SVerticalBox> LayerList = SNew(SVerticalBox)
	+SVerticalBox::Slot()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			SNew(SButton)
			.Text(FText::FromString("Ok"))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.ButtonStyle(EditorStyle, "PropertyEditor.AssetComboStyle")
			.ForegroundColor(EditorStyle.GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
			.OnClicked(FOnClicked::CreateLambda([&SelectionValid,Window]()
			{
				SelectionValid=true;
				Window->RequestDestroyWindow();
				return FReply::Handled();
			}))
		]
		+SHorizontalBox::Slot()
		[
			SNew(SButton)
			.Text(FText::FromString("Cancel"))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.ButtonStyle(EditorStyle, "PropertyEditor.AssetComboStyle")
			.ForegroundColor(EditorStyle.GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
			.OnClicked(FOnClicked::CreateLambda([&SelectionValid,Window]()
			{
				Window->RequestDestroyWindow();
				return FReply::Handled();
			}))
		]
	]
	+SVerticalBox::Slot()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([this]()
			{
				return this->bIsolate?ECheckBoxState::Checked:ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
			{
				this->bIsolate=NewState==ECheckBoxState::Checked;
			})
		]
		+SHorizontalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromString("Isolate"))
		]
	];

	TSet<FName> LayersSelected;
	for (TWeakObjectPtr<ULayer> Layer : AvailableLayers) {
		TSharedPtr<SCheckBox> Checkbox;
		LayerList->AddSlot()
			[
				
			
				SNew(SButton)
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				.VAlign(EVerticalAlignment::VAlign_Fill)
				.ButtonStyle(EditorStyle, "PropertyEditor.AssetComboStyle")
				.ForegroundColor(EditorStyle.GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
				[
					SNew(SHorizontalBox)
					
					+SHorizontalBox::Slot()
					.HAlign(EHorizontalAlignment::HAlign_Left)
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						SAssignNew(Checkbox,SCheckBox)
						.Visibility(EVisibility::HitTestInvisible)
					]
					+SHorizontalBox::Slot()
					.HAlign(EHorizontalAlignment::HAlign_Fill)
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						SNew(STextBlock)
						.Text(
							FText::FromName(Layer->GetLayerName())

							)
						.Justification(ETextJustify::Left)
					]
				]
				.OnClicked_Lambda([Layer,&LayersSelected, Checkbox]()->FReply
				{
					Checkbox->ToggleCheckedState();
					if (Checkbox->IsChecked())
					{
						LayersSelected.Add(Layer->GetLayerName());
					}
					else
					{
						LayersSelected.Remove(Layer->GetLayerName());
					}
					return FReply::Handled();
				})
			];
	}

	Window->SetContent(LayerList);

	GEditor->EditorAddModalWindow(Window);
	Layers=LayersSelected.Array();
	return true;
}

void UShowLayersCommand::Execute()
{
	ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();
	if (IsValid(LayersSubsystem))
	{
		TArray<FName> AllLayersName;
		TArray<FName> Visible;
		TArray<FName> Invisible;
		LayersSubsystem->AddAllLayerNamesTo(AllLayersName);
		for (FName LayerName:AllLayersName)
		{
			if (Layers.Contains(LayerName))
			{
				Visible.Add(LayerName);
				
			}else
			{
				if (bIsolate)
				{
					Invisible.Add(LayerName);
				}
			}
		}
		LayersSubsystem->SetLayersVisibility(Visible,true);
		LayersSubsystem->SetLayersVisibility(Invisible,false);
	}
	return;
}
