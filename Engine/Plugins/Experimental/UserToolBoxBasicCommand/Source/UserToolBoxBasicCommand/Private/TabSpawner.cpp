// Copyright Epic Games, Inc. All Rights Reserved.


#include "TabSpawner.h"
#include "Framework/Docking/TabManager.h"
UTabSpawner::UTabSpawner()
{
	Name="Tab Spawner";
	Tooltip="Allow you to spawn any tab by name";
	Category="Utility";
}

void UTabSpawner::Execute()
{
	FGlobalTabmanager::Get()->TryInvokeTab(TabName);
}
