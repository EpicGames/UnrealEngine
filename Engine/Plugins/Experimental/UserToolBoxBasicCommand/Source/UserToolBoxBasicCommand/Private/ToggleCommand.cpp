// Copyright Epic Games, Inc. All Rights Reserved.


#include "ToggleCommand.h"
#include "Engine/Selection.h"
void UToggleCommand::Execute()
{
	if (Commands.IsEmpty())
	{
		return;
	}
	if (CurrentIndex>=Commands.Num())
	{
		CurrentIndex=0;
	}
	UUTBBaseCommand* Current=Commands[CurrentIndex];
	CurrentIndex++;
	if (!IsValid(Current))
	{
		return;
	}
	Current->ExecuteCommand();
}
