// Copyright Epic Games, Inc. All Rights Reserved.


#include "ToggleCommandInline.h"


#include "EditorUtilityBlueprint.h"
#include "EngineUtils.h"
#include "UserToolBoxBaseBlueprint.h"
#include "UTBBaseTab.h"
#include "Engine/Selection.h"
#include "UserToolBoxBasicCommand.h"



void UToggleCommandInline::Execute()
{
	if (Commands.IsEmpty())
	{
		return;
	}
	if (CurrentIndex>=Commands.Num())
	{
		CurrentIndex=0;
	}
	UUTBBaseCommand* Current=Commands[CurrentIndex];
	CurrentIndex++;
	if (!IsValid(Current))
	{
		return;
	}
	Current->ExecuteCommand();

}
