// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserToolBoxBasicCommand.h"

#include "AssetToolsModule.h"
//#include "ExecuteBindableAction.h"
//#include "IAssetTools.h"

#include "SelectActorByFilter.h"
#include "ToggleCommand.h"
#include "Customization/BaseCompositeCommandCustomization.h"
#include "Customization/BindableActionCustomization.h"

#include "UObject/CoreRedirects.h"
#include "PropertyEditorModule.h"
#define LOCTEXT_NAMESPACE "FUserToolBox_BasicCommandModule"
DEFINE_LOG_CATEGORY(LogUserToolBoxBasicCommand);
void FUserToolBox_BasicCommandModule::StartupModule()
{

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomClassLayout(UBaseCompositeCommand::StaticClass()->GetFName(),FOnGetDetailCustomizationInstance::CreateStatic(&FBaseCompositeCommandCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(FBindableActionInfo::StaticStruct()->GetFName(),FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBindableActionPropertyCustomization::MakeInstance));
}

void FUserToolBox_BasicCommandModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FUserToolBox_BasicCommandModule, UserToolBoxBasicCommand)