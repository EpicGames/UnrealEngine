// Copyright Epic Games, Inc. All Rights Reserved.


#include "ZoomAll.h"

#include "GameFramework/Actor.h"
#include "EngineUtils.h"
#include "EditorViewportClient.h"

UZoomAll::UZoomAll()
{
	Name="Zoom All";
	Tooltip="Zoom the camera to see the whole scene";
	Category="Viewport";
}

void UZoomAll::Execute()
{
	UWorld* World = GEngine->GetWorldContexts()[0].World();

	TActorIterator<AActor> Actor(World, AActor::StaticClass());
	FBox BoundingBox = Actor->GetComponentsBoundingBox();

	for (; Actor; ++Actor)
	{
		if (*Actor != nullptr)
		{
			BoundingBox += Actor->GetComponentsBoundingBox();
		}
	}

	FEditorViewportClient* EditorViewportClient = static_cast<FEditorViewportClient*>(GEditor->GetActiveViewport()->GetClient());
	EditorViewportClient->FocusViewportOnBox(BoundingBox);

	
}
