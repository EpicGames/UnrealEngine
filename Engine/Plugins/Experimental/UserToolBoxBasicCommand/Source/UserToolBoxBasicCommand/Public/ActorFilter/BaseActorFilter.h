// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "BaseActorFilter.generated.h"

/**
 * 
 */
UCLASS(Blueprintable,EditInlineNew, Abstract)
class USERTOOLBOXBASICCOMMAND_API UBaseActorFilter : public UObject
{
	GENERATED_BODY()
	public:
	
	UFUNCTION(BlueprintNativeEvent)
	TArray<AActor*> Filter(const TArray<AActor*>& Source);
	UFUNCTION(BlueprintNativeEvent)
	bool FilterUnit(AActor* Source);
	UPROPERTY(EditAnywhere,Category="FilterCommand")
	bool bShouldRunGlobalFilter=true;
	UPROPERTY(EditAnywhere,Category="FilterCommand")
	bool bShouldRunUnitFilter=true;
	virtual TArray<AActor*> FilterImpl(const TArray<AActor*>& Source)
	{
		TArray<AActor*> GroupFilter;
		TArray<AActor*> Result;
		GroupFilter.Reserve(Source.Num());
		GroupFilter=bShouldRunGlobalFilter?Filter(Source):Source;
	
		Result.Reserve(GroupFilter.Num());
		if (bShouldRunUnitFilter)
		{
			for (AActor* ActorToFilter:GroupFilter)
			{
				if (FilterUnit(ActorToFilter))
				{
					Result.Add(ActorToFilter);
				}
			}
		}
		else
		{
			Result=GroupFilter;
		}
		Result.Shrink();
		return Result;
	}
};


UCLASS(Blueprintable)
class UGetAllDescendants : public UBaseActorFilter
{
	GENERATED_BODY()
public:
	virtual TArray<AActor*> FilterImpl(const TArray<AActor*>& Source) override;
};



UCLASS(Blueprintable)
class UGetParents : public UBaseActorFilter
{
	GENERATED_BODY()
public:
	virtual TArray<AActor*> FilterImpl(const TArray<AActor*>& Source) override;
};
UCLASS(Blueprintable)
class UHasAttachedActor : public UBaseActorFilter
{
	GENERATED_BODY()
public:
	virtual TArray<AActor*> FilterImpl(const TArray<AActor*>& Source) override;
};

UCLASS(Blueprintable)
class UHasComponentOfClass : public UBaseActorFilter
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Filter")
	TSubclassOf<UActorComponent> ComponentClass;
	virtual TArray<AActor*> FilterImpl(const TArray<AActor*>& Source) override;
};
UCLASS(Blueprintable)
class UHasMetadataByKey : public UBaseActorFilter
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Filter")
	FName Key;
	virtual TArray<AActor*> FilterImpl(const TArray<AActor*>& Source) override;
};
UCLASS(Blueprintable)
class UHasMetadataByKeyAndValue : public UBaseActorFilter
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Filter")
	FName Key;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Filter")
	FString Value;
	virtual TArray<AActor*> FilterImpl(const TArray<AActor*>& Source) override;
};
UCLASS(Blueprintable)
class UIsClassOf : public UBaseActorFilter
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Filter")
	TSubclassOf<AActor>	ActorClass;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Filter")
	bool ChildClass=false;
	virtual TArray<AActor*> FilterImpl(const TArray<AActor*>& Source) override;
};
UCLASS(Blueprintable)
class UGetNDescendants : public UBaseActorFilter
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Filter")
	int		N;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Filter")
	bool	AddIntermediaries;
	virtual TArray<AActor*> FilterImpl(const TArray<AActor*>& Source) override;
};
UCLASS(Blueprintable)
class UHasMetadataByKeyAndValueDropDown : public UBaseActorFilter
{
	GENERATED_BODY()
public:
	
	virtual TArray<AActor*> FilterImpl(const TArray<AActor*>& Source) override;
};
UCLASS(Blueprintable)
class UEmptyStaticMeshActor : public UBaseActorFilter
{
	GENERATED_BODY()
public:
	
	virtual TArray<AActor*> FilterImpl(const TArray<AActor*>& Source) override;
};
