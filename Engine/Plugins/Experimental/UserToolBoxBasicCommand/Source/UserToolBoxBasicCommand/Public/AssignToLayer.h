// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "Layers/Layer.h"
#include "AssignToLayer.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UAssignToLayer : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:

	void Execute() override;
	virtual FName GetLayerName(TWeakObjectPtr<ULayer> Layer);
	UAssignToLayer();
};
