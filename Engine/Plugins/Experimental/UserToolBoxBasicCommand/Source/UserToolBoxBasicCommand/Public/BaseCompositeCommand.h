// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "BaseCompositeCommand.generated.h"

/**
 * 
 */
UCLASS(Abstract)
class USERTOOLBOXBASICCOMMAND_API UBaseCompositeCommand : public UUTBBaseCommand
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere,BlueprintReadWrite, Category="Composite Commands")
	TArray<TObjectPtr<UUTBBaseCommand>>		Commands;
	
	UBaseCompositeCommand()
	{
		Name="Toggle Command";
		Tooltip="This command let you execute sequentially some already registered command, one at a time (placeholder section is a good place to register these commands)";
		Category="Builtin";
	}

	virtual UUTBBaseCommand* CopyCommand(UObject* Owner) const override;

};
