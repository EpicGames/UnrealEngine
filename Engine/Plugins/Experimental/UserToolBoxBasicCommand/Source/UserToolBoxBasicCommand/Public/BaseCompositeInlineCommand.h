// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "BaseCompositeInlineCommand.generated.h"

/**
 * 
 */
UCLASS(Abstract, EditInlineNew)
class USERTOOLBOXBASICCOMMAND_API UBaseCompositeInlineCommand : public UUTBBaseCommand
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere,BlueprintReadWrite,Instanced,Category="CompositeCommand")
	TArray<TObjectPtr<UUTBBaseCommand>>		Commands;
	
	UBaseCompositeInlineCommand()
	{
		Name="Toggle Command";
		Tooltip="This command let you execute sequentially some already registered command, one at a time (placeholder section is a good place to register these commands)";
		Category="Builtin";
	}

	

};
