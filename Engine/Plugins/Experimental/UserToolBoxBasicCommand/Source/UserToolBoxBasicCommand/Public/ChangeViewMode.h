// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "ChangeViewMode.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UChangeViewMode : public UUTBBaseCommand
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere,BlueprintReadWrite,Category="ChangeViewCommand")
	TEnumAsByte<EViewModeIndex> ViewMode=EViewModeIndex::VMI_Lit;
	virtual void Execute() override;
	UChangeViewMode();
};
