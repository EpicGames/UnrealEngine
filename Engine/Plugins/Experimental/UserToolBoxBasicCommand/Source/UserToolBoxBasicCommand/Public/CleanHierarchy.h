// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "CleanHierarchy.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UCleanHierarchy : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:
	UPROPERTY(EditAnywhere,Category="CleanHierarchyCommand")
	TArray<FString> DSMetadataToPreserve;
	UPROPERTY(EditAnywhere,Category="CleanHierarchyCommand")
	bool RemoveEmptyBranches;
	UPROPERTY(EditAnywhere,Category="CleanHierarchyCommand")
	bool RemoveIntermediaryActorsWithoutGeometry;
	UCleanHierarchy();
	virtual void Execute() override;
};
