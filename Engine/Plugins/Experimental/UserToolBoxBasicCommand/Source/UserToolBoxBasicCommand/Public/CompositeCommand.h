// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseCompositeCommand.h"
#include "UTBBaseCommand.h"
#include "CompositeCommand.generated.h"

/**
 * 
 */

UCLASS(Blueprintable)
class USERTOOLBOXBASICCOMMAND_API UCompositeCommand : public UBaseCompositeCommand
{
	GENERATED_BODY()
	public:


	virtual void Execute() override;
	UCompositeCommand()
	{
		Name="Composite Command";
		Tooltip="This command let you execute sequentially other already set command (the placeholder section is a good place to register these commands";
		Category="Utility";
	}


};
