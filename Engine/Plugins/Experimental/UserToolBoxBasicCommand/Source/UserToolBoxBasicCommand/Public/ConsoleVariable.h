// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "ConsoleVariable.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UConsoleVariable : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:
	UPROPERTY(EditAnywhere,BlueprintReadWrite,Category="Console Command")
	TArray<FString> ConsoleCommands;
	
	virtual void Execute() override;

	UConsoleVariable()
	{
		Name="Console Variable Command";
		Tooltip="This console command let you execute any console variable/command";
		Category="Utility";
	}
};
