// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "EngineCommand.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UEngineCommand : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:

	UPROPERTY(EditAnywhere,BlueprintReadWrite,Category="EngineCommand")
	FString Command;
	virtual void Execute() override;
	UEngineCommand()
	{
		Name="Engine Command";
		Tooltip="This command let you execute any engine command";
		Category="Utility";
	}
};
