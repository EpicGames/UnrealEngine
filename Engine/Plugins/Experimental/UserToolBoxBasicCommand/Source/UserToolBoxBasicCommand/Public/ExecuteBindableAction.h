// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"

#include "ExecuteBindableAction.generated.h"

USTRUCT()
struct USERTOOLBOXBASICCOMMAND_API FBindableActionInfo
{
	GENERATED_BODY()
	UPROPERTY()
	FString Context;
	UPROPERTY()
	FString CommandName;
};

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UExecuteBindableAction : public UUTBBaseCommand
{
	GENERATED_BODY()
public:
	UExecuteBindableAction() ;

	UPROPERTY(EditAnywhere,Category="Bindable Command")
	FBindableActionInfo	CommandInfo;


public:
	virtual void Execute() override;
	
};
