// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"

#include "ExecutePythonScript.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UExecutePythonScript : public UUTBBaseCommand
{
	GENERATED_BODY()
public:
	UExecutePythonScript() ;
private:
	UPROPERTY(EditAnywhere,Category="Python Command")
	FFilePath	ScriptPath;

	UPROPERTY(EditAnywhere,Category="Python Command")
	FString		Args;

public:
	virtual void Execute() override;
};
