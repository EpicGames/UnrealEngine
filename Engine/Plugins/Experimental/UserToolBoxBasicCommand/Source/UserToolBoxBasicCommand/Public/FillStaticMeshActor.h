// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "FillStaticMeshActor.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UFillStaticMeshActor : public UUTBBaseCommand
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere,BlueprintReadWrite,Category="FillStaticMeshActor")
	TArray<FString>	RootPaths;
	
	UPROPERTY(EditAnywhere,BlueprintReadWrite,Category="FillStaticMeshActor")
	bool	bAffectOnlyEmptyStaticMeshActor=true;
	
	void Execute() override;
	UFillStaticMeshActor();
};
