// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "FlipNormals.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UFlipNormals : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:

	virtual void Execute() override;
	UFlipNormals();
};
