// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "IsolateSelection.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UIsolateSelection : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:
	UPROPERTY(EditAnywhere,Category="Isolate Command")
	bool bShouldOnlyAffectStaticMeshActor=false;
	UIsolateSelection();
	virtual void Execute() override;
};
