// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "MergeCommand.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UMerge : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:

	UPROPERTY(EditAnywhere,BlueprintReadWrite,Category="Merge Command")
	bool Advanced;
	UMerge();
	virtual void Execute() override;
};
