// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "MirrorActorCommand.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UMirrorActorCommand : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:
	UPROPERTY(EditAnywhere,BlueprintReadWrite,Category="MirrorCommand")
	bool XAxis=false;
	UPROPERTY(EditAnywhere,BlueprintReadWrite,Category="MirrorCommand")
	bool YAxis=false;
	UPROPERTY(EditAnywhere,BlueprintReadWrite,Category="MirrorCommand")
	bool ZAxis=false;
	UMirrorActorCommand();
	virtual void Execute() override;
};
