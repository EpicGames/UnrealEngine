// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "ActorFilter/BaseActorFilter.h"
#include "SelectActorByFilter.generated.h"


UENUM()
enum class EActorFilterRule : uint8
{
	Add =0,
	Intersect =2,
	Substract =3,
	Replace= 4 
};
UENUM()
enum class EActorFilterSource : uint8
{
	OriginalSelection =0,
	PreviousResult =1
};


USTRUCT(BlueprintType)
struct FActorFilterOptions
{
	GENERATED_BODY()
	UPROPERTY(EditAnywhere, Category="Actor Filter",Instanced)
	TObjectPtr<UBaseActorFilter> Filter;
	UPROPERTY(EditAnywhere, Category="Actor Filter")
	EActorFilterRule Rule=EActorFilterRule::Replace;
	UPROPERTY(EditAnywhere, Category="Actor Filter")
	EActorFilterSource Source=EActorFilterSource::PreviousResult;
	
	UPROPERTY(EditAnywhere, Category="Actor Filter")
	bool ShowProperties=false;
};


UCLASS(Blueprintable)
class USERTOOLBOXBASICCOMMAND_API USelectActorByFilter : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:
	UPROPERTY(EditAnywhere,BlueprintReadOnly, Category="Actor Filter")
	bool	ApplyToCurrentSelection;
	UPROPERTY(EditAnywhere,BlueprintReadWrite, Category="Actor Filter")
	TArray<FActorFilterOptions>	FilterStack;
	virtual void Execute() override;
	USelectActorByFilter();
	
	
};