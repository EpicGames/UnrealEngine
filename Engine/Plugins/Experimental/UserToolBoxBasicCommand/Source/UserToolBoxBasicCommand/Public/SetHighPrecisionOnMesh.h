// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "SetHighPrecisionOnMesh.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API USetHighPrecisionOnMesh : public UUTBBaseCommand
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite,Category="HighPrecisionCommand")
	bool bHighPrecisionTangent=true;
	UPROPERTY(EditAnywhere, BlueprintReadWrite,Category="HighPrecisionCommand")
	bool bHighPrecisionUV=true;
	USetHighPrecisionOnMesh();
	virtual void Execute() override;
};
