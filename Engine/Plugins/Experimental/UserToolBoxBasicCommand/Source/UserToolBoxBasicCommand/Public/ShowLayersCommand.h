// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "ShowLayersCommand.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UShowLayersCommand : public UUTBBaseCommand
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere,BlueprintReadWrite,Category="LayerCommand")
	TArray<FName>	Layers;

	UPROPERTY(EditAnywhere,BlueprintReadWrite,Category="LayerCommand")
	bool bIsolate=false;
	UShowLayersCommand();
	virtual bool DisplayParameters() override;
	virtual void Execute() override;
};
