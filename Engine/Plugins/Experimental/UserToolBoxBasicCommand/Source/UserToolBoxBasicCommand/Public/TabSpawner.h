// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "TabSpawner.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UTabSpawner : public UUTBBaseCommand
{
	GENERATED_BODY()
	UTabSpawner();
	UPROPERTY(EditAnywhere,Category="TabSpawnerCommand")
	FName TabName;
public:
	virtual void Execute() override;
};
