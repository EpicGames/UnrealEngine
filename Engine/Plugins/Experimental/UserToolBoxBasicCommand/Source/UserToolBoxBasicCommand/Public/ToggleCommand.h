// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseCompositeCommand.h"
#include "UTBBaseCommand.h"
#include "CompositeCommand.h"
#include "ToggleCommand.generated.h"

/**
 * 
 */



UCLASS(Blueprintable)
class USERTOOLBOXBASICCOMMAND_API UToggleCommand : public UBaseCompositeCommand
{
	GENERATED_BODY()
	public:
	UToggleCommand()
	{
		Name="Toggle Command";
		Tooltip="This command let you execute sequentially some already registered command, one at a time (placeholder section is a good place to register these commands)";
		Category="Utility";
	}


private:
	int CurrentIndex=0;
	virtual void Execute() override;

};
