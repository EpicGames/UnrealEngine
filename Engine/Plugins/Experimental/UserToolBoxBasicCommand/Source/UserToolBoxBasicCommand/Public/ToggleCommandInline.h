// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "CompositeCommand.h"
#include "ToggleCommandInline.generated.h"

/**
 * 
 */



UCLASS(Blueprintable)
class USERTOOLBOXBASICCOMMAND_API UToggleCommandInline : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:

	UPROPERTY(EditAnywhere,BlueprintReadWrite,Instanced,Category="ToggleCommand")
	TArray<TObjectPtr<UUTBBaseCommand>>		Commands;
	
	UToggleCommandInline()
	{
		Name="Toggle Command Inline";
		Tooltip="This command let you execute sequentially some already registered command, one at a time (placeholder section is a good place to register these commands)";
		Category="Utility";
	}


private:
	int CurrentIndex=0;
	virtual void Execute() override;

};
