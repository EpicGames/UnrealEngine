// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "ZoomAll.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXBASICCOMMAND_API UZoomAll : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:
	UZoomAll();
	virtual void Execute() override;
	
};
