// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UserToolBoxBasicCommand : ModuleRules
{
	public UserToolBoxBasicCommand(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core", "AssetTools"
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"UserToolBoxCore",
				"EditorScriptingUtilities",
				"UnrealEd",
				"EditorStyle",
				 "Blutility", "DatasmithContent",
				 "EditorInteractiveToolsFramework",
				 "DatasmithContent",
				 "EditorWidgets",
				 "ToolWidgets",
				 "InputCore",
				 "PropertyEditor"

				 // ... add private dependencies that you statically link with here ...	
			}
			);
		
		if (Target.Version.MajorVersion >= 5)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"StaticMeshEditor","GeometryFramework","GeometryScriptingCore"
				});
		}
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
