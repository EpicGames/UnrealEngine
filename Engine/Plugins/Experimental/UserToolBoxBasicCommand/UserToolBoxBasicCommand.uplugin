{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.10",
	"FriendlyName": "UserToolBoxBasicCommand",
	"Description": "Basic set of command to populate a custom editor tab",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "UserToolBoxBasicCommand",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins":
	[
		{
			"Name": "UserToolBoxCore",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled" : true
		},
		{
			"Name": "DatasmithContent",
			"Enabled" : true
		}
		,
		  {
			"Name": "GeometryScripting",
			"Enabled" : true
		  }
	]
}